{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Niagara UI Renderer",
	"Description": "Renders Niagara CPU particle systems inside Slate/UMG widgets using a dedicated UI sprite renderer.",
	"Category": "FX",
	"CreatedBy": "Epic Games",
	"CreatedByURL": "",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "NiagaraUIRenderer",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "NiagaraUIRendererEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "Niagara",
			"Enabled": true
		}
	]
}
