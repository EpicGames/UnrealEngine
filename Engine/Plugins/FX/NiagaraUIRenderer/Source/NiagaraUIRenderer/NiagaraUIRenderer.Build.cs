// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class NiagaraUIRenderer : ModuleRules
{
	public NiagaraUIRenderer(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"CoreUObject",
			"Engine",
			"Niagara",
			"NiagaraCore",
			"SlateCore",
			"Slate",
			"UMG",
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"RenderCore",
			"RHI",
		});
	}
}
