// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraUIComponent.h"
#include "NiagaraUIRendererProperties.h"

#include "NiagaraSystemInstanceController.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraEmitterInstance.h"

UNiagaraUIComponent::UNiagaraUIComponent()
{
	SetAllowScalability(false);
	SetForceSolo(true);
	SetAgeUpdateMode(ENiagaraAgeUpdateMode::TickDeltaTime);
	SetCanRenderWhileSeeking(true);
	SetMaxSimTime(0.0f);
	SetAutoDestroy(false);
	SetAutoActivate(false);
	PrimaryComponentTick.bStartWithTickEnabled = false;
}

FPrimitiveSceneProxy* UNiagaraUIComponent::CreateSceneProxy()
{
	return nullptr;
}

const FNiagaraUIRenderData* UNiagaraUIComponent::GetRenderData() const
{
	return RenderData.Get();
}

void UNiagaraUIComponent::TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	CreateRenderData();
}

void UNiagaraUIComponent::CreateRenderData()
{
	FNiagaraSystemInstanceControllerPtr Controller = GetSystemInstanceController();
	if (!Controller || !Controller->IsValid())
	{
		return;
	}

	FNiagaraSystemInstance* Instance = Controller->GetSystemInstance_Unsafe();
	if (!Instance)
	{
		return;
	}

	TUniquePtr<FNiagaraUIRenderData> NewRenderData(new FNiagaraUIRenderData());
	for (const FNiagaraEmitterInstanceRef& EmitterRef : Instance->GetEmitters())
	{
		FNiagaraEmitterInstance& EmitterInstance = *EmitterRef;

		EmitterInstance.ForEachEnabledRenderer(
			[&](const UNiagaraRendererProperties* Props)
			{
				const UNiagaraUIRendererProperties* UIProperties = Cast<const UNiagaraUIRendererProperties>(Props);
				FNiagaraUIRendererRenderData* RendererRenderData = UIProperties ? UIProperties->CreateRenderData(EmitterInstance) : nullptr;
				if (RendererRenderData)
				{
					check(RendererRenderData->WeakProperties.Get() != nullptr);

					NewRenderData->RendererRenderDatas.Emplace(RendererRenderData);
				}
			}
		);
	}

	if (NewRenderData->RendererRenderDatas.Num() > 0)
	{
		NewRenderData->RendererRenderDatas.StableSort([](const TUniquePtr<FNiagaraUIRendererRenderData>& DataA, const TUniquePtr<FNiagaraUIRendererRenderData>& DataB) { return DataA->SortOrder < DataB->SortOrder; });
	}

	Swap(RenderData, NewRenderData);
}
