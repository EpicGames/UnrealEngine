// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraUIRenderContext.h"

#include "Materials/MaterialInterface.h"
#include "Rendering/DrawElementTypes.h"
#include "SlateMaterialBrush.h"

FNiagaraUIRenderContext::FNiagaraUIRenderContext(FMaterialCacheMap& InCacheMap, const FGeometry& InAllottedGeometry)
	: MaterialHandleCache(InCacheMap)
	, AllottedGeometry(InAllottedGeometry)
{
	const FVector2D GeometrySize = AllottedGeometry.GetLocalSize();
	ScreenOrigin.X = float(GeometrySize.X) * 0.5f;
	ScreenOrigin.Y = float(GeometrySize.Y) * 0.5f;
}

FNiagaraUIRendererScratchBuffers FNiagaraUIRenderContext::GetScratchBuffers(int32 NumVertices, int32 NumInidices) const
{
	ScratchVertices.SetNumUninitialized(NumVertices, EAllowShrinking::No);
	ScratchIndices.SetNumUninitialized(NumInidices, EAllowShrinking::No);

	return FNiagaraUIRendererScratchBuffers { ScratchVertices, ScratchIndices };
}

TConstArrayView<int32> FNiagaraUIRenderContext::CreateParticleOrderTable(int32 NumInstances, ENiagaraSortMode SortMode, FNiagaraDataSetReaderFloat<FNiagaraPosition> PositionReader, FNiagaraDataSetReaderFloat<float> SortOrderReader, int32 RendererVisibility, FNiagaraDataSetReaderInt32<int32> VisTagReader) const
{
	ParticleOrderTable.SetNumUninitialized(NumInstances, EAllowShrinking::No);

	// Generate order table using vis tag to remove entries
	{
		int32 NumCulledInstances = 0;
		for (int32 i=0; i < NumInstances; ++i)
		{
			if (VisTagReader.GetSafe(i, RendererVisibility) == RendererVisibility)
			{
				ParticleOrderTable[NumCulledInstances] = i;
				++NumCulledInstances;
			}
		}
		if (NumCulledInstances == 0)
		{
			return {};
		}

		NumInstances = NumCulledInstances;
		ParticleOrderTable.SetNumUninitialized(NumCulledInstances);
	}

	// Sort the data if it's bound
	switch (SortMode)
	{
		case ENiagaraSortMode::ViewDepth:
		case ENiagaraSortMode::ViewDistance:
			if (PositionReader.IsValid())
			{
				int32 DepthComponent = 0;
				switch (ScreenPlane)
				{
					case ENiagaraUIScreenPlane::XY: DepthComponent = 2; break;
					case ENiagaraUIScreenPlane::XZ: DepthComponent = 1; break;
					case ENiagaraUIScreenPlane::YZ: DepthComponent = 0; break;
				}

				ParticleOrderTable.Sort(
					[&](int32 IndexA, const int32 IndexB)
					{
						return PositionReader.Get(IndexA).Component(DepthComponent) < PositionReader.Get(IndexB).Component(DepthComponent);
					}
				);
			}
			break;

		case ENiagaraSortMode::CustomAscending:
			if (SortOrderReader.IsValid())
			{
				ParticleOrderTable.Sort(
					[&](int32 IndexA, const int32 IndexB)
					{
						return SortOrderReader.Get(IndexA) < SortOrderReader.Get(IndexB);
					}
				);
			}
			break;

		case ENiagaraSortMode::CustomDecending:
			if (SortOrderReader.IsValid())
			{
				ParticleOrderTable.Sort(
					[&](int32 IndexA, const int32 IndexB)
					{
						return SortOrderReader.Get(IndexA) > SortOrderReader.Get(IndexB);
					}
				);
			}
			break;
	}

	return ParticleOrderTable;
}

void FNiagaraUIRenderContext::DrawCustomVerts(const UMaterialInterface* Material) const
{
	DrawCustomVerts(Material, ScratchVertices, ScratchIndices);

	ScratchVertices.Reset();
	ScratchIndices.Reset();
}

const FSlateResourceHandle& FNiagaraUIRenderContext::GetOrCreateMaterialHandle(const UMaterialInterface* Material) const
{
	check(IsInGameThread());

	// Remove stale entries?
	FObjectKey MaterialKey(Material);
	if (const FSlateResourceHandle* Existing = MaterialHandleCache.Find(MaterialKey))
	{
		if (Existing->IsValid())
		{
			return *Existing;
		}
		MaterialHandleCache.Remove(MaterialKey);
	}

	if (!FSlateApplication::IsInitialized())
	{
		static FSlateResourceHandle Invalid;
		return Invalid;
	}

	// FSlateMaterialBrush doesn't take a const UMaterialInterface
	FSlateMaterialBrush Brush(*const_cast<UMaterialInterface*>(Material), FVector2D(1.0, 1.0));
	FSlateResourceHandle NewHandle = FSlateApplication::Get().GetRenderer()->GetResourceHandle(Brush);

	return MaterialHandleCache.Add(MaterialKey, MoveTemp(NewHandle));
}

void FNiagaraUIRenderContext::SetScreenOriginAlignment(EHorizontalAlignment HorizontalAlignment, EVerticalAlignment VerticalAlignment)
{
	const FVector2D GeometrySize = AllottedGeometry.GetLocalSize();
	switch (HorizontalAlignment)
	{
		case HAlign_Left:	ScreenOrigin.X = 0.0f; break;
		case HAlign_Center:	ScreenOrigin.X = float(GeometrySize.X) * 0.5f; break;
		case HAlign_Right:	ScreenOrigin.X = float(GeometrySize.X); break;
	}

	switch (VerticalAlignment)
	{
		case VAlign_Top:	ScreenOrigin.Y = 0.0f; break;
		case VAlign_Center:	ScreenOrigin.Y = float(GeometrySize.Y) * 0.5f; break;
		case VAlign_Bottom:	ScreenOrigin.Y = float(GeometrySize.Y); break;
	}
}
