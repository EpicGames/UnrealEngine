// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "NiagaraUISpriteRendererProperties.h"

class FNiagaraUIRendererModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
		// Process any UNiagaraUISpriteRendererProperties instances that tried to
		// InitBindings() before the Niagara module was loaded.
		UNiagaraUISpriteRendererProperties::ProcessDeferredInit();
	}

	virtual void ShutdownModule() override
	{
	}
};

IMPLEMENT_MODULE(FNiagaraUIRendererModule, NiagaraUIRenderer)
