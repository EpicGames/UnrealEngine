// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraUISlateRenderContext.h"

#include "Rendering/DrawElementTypes.h"

FNiagaraUISlateRenderContext::FNiagaraUISlateRenderContext(FMaterialCacheMap& InCacheMap, const FGeometry& InAllottedGeometry, FSlateWindowElementList& InOutDrawElements)
	: FNiagaraUIRenderContext(InCacheMap, InAllottedGeometry)
	, OutDrawElements(InOutDrawElements)
{
}

void FNiagaraUISlateRenderContext::DrawCustomVerts(const UMaterialInterface* Material, const TArray<FSlateVertex>& Vertices, const TArray<SlateIndex>& Indices) const
{
	if (!Material)
	{
		//-TODO: Use default material??
		return;
	}

	const FSlateResourceHandle& Handle = GetOrCreateMaterialHandle(Material);
	if (!Handle.IsValid())
	{
		return;
	}

	FSlateDrawElement::MakeCustomVerts(OutDrawElements, LayerId, Handle, Vertices, Indices, nullptr, 0, 0, DrawEffect);
}

