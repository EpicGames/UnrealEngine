// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraUIRenderContext.h"

class FNiagaraUISlateRenderContext : public FNiagaraUIRenderContext
{
public:
	explicit FNiagaraUISlateRenderContext(FMaterialCacheMap& InCacheMap, const FGeometry& InAllottedGeometry, FSlateWindowElementList& InOutDrawElements);
	virtual ~FNiagaraUISlateRenderContext() = default;

	virtual void DrawCustomVerts(const UMaterialInterface* Material, const TArray<FSlateVertex>& Vertices, const TArray<SlateIndex>& Indices) const override;

private:
	FSlateWindowElementList& OutDrawElements;
};
