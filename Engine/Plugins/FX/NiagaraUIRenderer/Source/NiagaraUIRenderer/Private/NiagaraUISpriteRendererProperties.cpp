// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraUISpriteRendererProperties.h"
#include "NiagaraUIRenderContext.h"
#include "NiagaraConstants.h"
#include "NiagaraEmitterInstance.h"

#include "Modules/ModuleManager.h"
#include "Widgets/Text/STextBlock.h"
#include "MaterialDomain.h"

#define LOCTEXT_NAMESPACE "NiagaraUIRenderer"

namespace NiagaraUISpriteRendererPropertiesPrivate
{
	static TArray<TWeakObjectPtr<UNiagaraUISpriteRendererProperties>> UISpriteRendererPropertiesToDeferredInit;
} //namespace NiagaraUISpriteRendererPropertiesPrivate

void UNiagaraUISpriteRendererProperties::ProcessDeferredInit()
{
	using namespace NiagaraUISpriteRendererPropertiesPrivate;

	for (TWeakObjectPtr<UNiagaraUISpriteRendererProperties>& Weak : UISpriteRendererPropertiesToDeferredInit)
	{
		if (UNiagaraUISpriteRendererProperties* Props = Weak.Get())
		{
			Props->InitBindings();
		}
	}
	UISpriteRendererPropertiesToDeferredInit.Empty();
}

UNiagaraUISpriteRendererProperties::UNiagaraUISpriteRendererProperties()
{
	AttributeBindings =
	{
		&PositionBinding,
		&ColorBinding,
		&SpriteSizeBinding,
		&SpriteRotationBinding,
		&SubImageIndexBinding,
		&DynamicMaterialParameterBinding,
		&CustomSortingBinding,
		&RendererVisibilityTagBinding,
	};
};

void UNiagaraUISpriteRendererProperties::PostInitProperties()
{
	using namespace NiagaraUISpriteRendererPropertiesPrivate;

	Super::PostInitProperties();

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		if (!FModuleManager::Get().IsModuleLoaded("Niagara"))
		{
			UISpriteRendererPropertiesToDeferredInit.Add(this);
			return;
		}
		InitBindings();
	}
}

void UNiagaraUISpriteRendererProperties::PostLoad()
{
	Super::PostLoad();

	// Ensure bindings have been set if this object was loaded from disk without them
	if (PositionBinding.GetParamMapBindableVariable().GetName() == NAME_None)
	{
		InitBindings();
	}
}

void UNiagaraUISpriteRendererProperties::InitBindings()
{
	if (PositionBinding.GetParamMapBindableVariable().GetName() != NAME_None)
	{
		// Already initialized
		return;
	}

	PositionBinding					= FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_POSITION);
	ColorBinding					= FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_COLOR);
	SpriteSizeBinding				= FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_SPRITE_SIZE);
	SpriteRotationBinding			= FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_SPRITE_ROTATION);
	SubImageIndexBinding			= FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_SUB_IMAGE_INDEX);
	DynamicMaterialParameterBinding	= FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM);
	CustomSortingBinding			= FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_AGE);
	RendererVisibilityTagBinding	= FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_VISIBILITY_TAG);
}

void UNiagaraUISpriteRendererProperties::GetUsedMaterials(const FNiagaraEmitterInstance* /*EmitterInstance*/, TArray<UMaterialInterface*>& OutMaterials) const
{
	if (Material)
	{
		OutMaterials.AddUnique(Material);
	}
}

#if WITH_EDITORONLY_DATA
void UNiagaraUISpriteRendererProperties::GetRendererWidgets(const FNiagaraEmitterInstance* InEmitter, TArray<TSharedPtr<SWidget>>& OutWidgets, TSharedPtr<FAssetThumbnailPool> InThumbnailPool) const
{
	TArray<UMaterialInterface*> Materials;
	GetUsedMaterials(InEmitter, Materials);
	CreateRendererWidgetsForAssets(Materials, InThumbnailPool, OutWidgets);
}

void UNiagaraUISpriteRendererProperties::GetRendererTooltipWidgets(const FNiagaraEmitterInstance* InEmitter, TArray<TSharedPtr<SWidget>>& OutWidgets, TSharedPtr<FAssetThumbnailPool> InThumbnailPool) const
{
	TArray<UMaterialInterface*> Materials;
	GetUsedMaterials(InEmitter, Materials);
	if (Materials.Num() > 0)
	{
		GetRendererWidgets(InEmitter, OutWidgets, InThumbnailPool);
	}
	else
	{
		OutWidgets.Add(
			SNew(STextBlock)
			.Text(LOCTEXT("UISpriteRendererNoMat", "UI Sprite Renderer (No Material Set)"))
		);
	}
}

void UNiagaraUISpriteRendererProperties::GetRendererFeedback(const FVersionedNiagaraEmitter& InEmitter, TArray<FNiagaraRendererFeedback>& OutErrors, TArray<FNiagaraRendererFeedback>& OutWarnings, TArray<FNiagaraRendererFeedback>& OutInfo) const
{
	Super::GetRendererFeedback(InEmitter, OutErrors, OutWarnings, OutInfo);

	if (Material == nullptr || Material->GetBaseMaterial() == nullptr)
	{
		OutErrors.Emplace(
			LOCTEXT("NoMaterial", "No material is assigned nothing will render."),
			LOCTEXT("NoMaterialSummary", "No Material Assigned.")
		);
	}
	else if (Material->GetBaseMaterial()->MaterialDomain != MD_UI)
	{
		OutErrors.Emplace(
			LOCTEXT("InvalidMaterial", "The current material is invalid as it is not a UI domain material."),
			LOCTEXT("InvalidMaterialSummary", "Invalid Material Assigned.")
		);
	}
}
#endif //WITH_EDITORONLY_DATA

void UNiagaraUISpriteRendererProperties::CacheFromCompiledData(const FNiagaraDataSetCompiledData* CompiledData)
{
	InitParticleDataSetAccessor(PositionDataSetAccessor, CompiledData, PositionBinding);
	InitParticleDataSetAccessor(ColorDataSetAccessor, CompiledData, ColorBinding);
	InitParticleDataSetAccessor(SpriteSizeDataSetAccessor, CompiledData, SpriteSizeBinding);
	InitParticleDataSetAccessor(SpriteRotationDataSetAccessor, CompiledData, SpriteRotationBinding);
	InitParticleDataSetAccessor(SubImageIndexDataSetAccessor, CompiledData, SubImageIndexBinding);
	InitParticleDataSetAccessor(DynamicMaterialParameterDataSetAccessor, CompiledData, DynamicMaterialParameterBinding);
	InitParticleDataSetAccessor(CustomSortingDataSetAccessor, CompiledData, CustomSortingBinding);
	InitParticleDataSetAccessor(RendererVisibilityTagDataSetAccessor, CompiledData, RendererVisibilityTagBinding);
}

FNiagaraUIRendererRenderData* UNiagaraUISpriteRendererProperties::CreateRenderData(const FNiagaraEmitterInstance& EmitterInstance) const
{
	const FNiagaraDataSet& ParicleDataSet = EmitterInstance.GetParticleData();
	FNiagaraDataBuffer* ParticleData = ParicleDataSet.GetCurrentData();
	if (!ParticleData || ParticleData->GetNumInstances() == 0)
	{
		return nullptr;
	}

	UMaterial* BaseMaterial = Material ? Material->GetMaterial() : nullptr;
	if (!BaseMaterial || BaseMaterial->MaterialDomain != MD_UI)
	{
		return nullptr;
	}

	FNiagaraUIRendererRenderData* RenderData = new FNiagaraUIRendererRenderData();
	RenderData->WeakProperties	= this;
	RenderData->DataBuffer		= ParticleData;
	RenderData->WeakMaterial	= Material;
	RenderData->SortOrder		= SortOrderHint;

	return RenderData;
}

void UNiagaraUISpriteRendererProperties::ExecuteRender(const FNiagaraUIRenderContext& RenderContext, const FNiagaraUIRendererRenderData& RendererRenderData) const
{
	const FNiagaraDataBuffer* ParticleData = RendererRenderData.DataBuffer;

	// Get readers
	auto PositionReader			= PositionDataSetAccessor.GetReader(ParticleData);
	auto ColorReader			= ColorDataSetAccessor.GetReader(ParticleData);
	auto SpriteSizeReader		= SpriteSizeDataSetAccessor.GetReader(ParticleData);
	auto SpriteRotationReader	= SpriteRotationDataSetAccessor.GetReader(ParticleData);
	auto SubImageIndexReader	= SubImageIndexDataSetAccessor.GetReader(ParticleData);
	auto DynamicParamReader		= DynamicMaterialParameterDataSetAccessor.GetReader(ParticleData);

	// Default values
	const FNiagaraPosition DefaultPosition	= FNiagaraPosition(0.0f, 0.0f, 0.0f);	//ParameterStore.GetParameterValueOrDefault(RendererProperties->PositionBinding.GetParamMapBindableVariable(), DefaultPos);
	const FLinearColor DefaultColor			= FLinearColor::White;					//SYS_PARAM_PARTICLES_COLOR.GetValue<FNiagaraPosition>();
	const FVector2f DefaultSpriteSize		= FVector2f(10.0f);						//SYS_PARAM_PARTICLES_SPRITE_SIZE.GetValue<FVector2f>();
	const float DefaultSpriteRotation		= 0.0f;									//SYS_PARAM_PARTICLES_SPRITE_ROTATION.GetValue<float>();
	const float DefaultSubImageIndex		= 0;									//SubImageIndexDataSetAccessor.GetReader(ParticleData);
	const FVector4f DefaultDynamicParam		= FVector4f::Zero();					//SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM.GetValue<FVector4f>();

	// Sub UV data
	const float SubUVInvCols = 1.f / FMath::Max(SubImageSize.X, 1.f);
	const float SubUVInvRows = 1.f / FMath::Max(SubImageSize.Y, 1.f);
	const int32 SubUVNumCols = FMath::Max(FMath::RoundToInt32(SubImageSize.X), 1);
	const int32 SubUVNumRows = FMath::Max(FMath::RoundToInt32(SubImageSize.Y), 1);
	const int32 SubUVNumFrames = SubUVNumCols * SubUVNumRows;

	// Generate particle indirection table (if needed)
	TConstArrayView<int32> ParticleOrderTable;
	if (SortMode != ENiagaraSortMode::None || RendererVisibilityTagDataSetAccessor.IsValid())
	{
		ParticleOrderTable = RenderContext.CreateParticleOrderTable(ParticleData->GetNumInstances(), SortMode, PositionReader, CustomSortingDataSetAccessor.GetReader(ParticleData), RendererVisibility, RendererVisibilityTagDataSetAccessor.GetReader(ParticleData));
		if (ParticleOrderTable.Num() == 0)
		{
			return;
		}
	}

	const uint32 NumInstances = ParticleOrderTable.Num() > 0 ? ParticleOrderTable.Num() : ParticleData->GetNumInstances();

	FNiagaraUIRendererScratchBuffers ScratchBuffers = RenderContext.GetScratchBuffers(NumInstances * 4, NumInstances * 6);
	ScratchBuffers.ScratchIndices.SetNumUninitialized(NumInstances * 6);
	ScratchBuffers.ScratchVertices.SetNumUninitialized(NumInstances * 4);

	for (uint32 iInstance = 0; iInstance < NumInstances; ++iInstance)
	{
		const uint32			iParticle			= ParticleOrderTable.Num() > 0 ? ParticleOrderTable[iInstance] : iInstance;
		const FNiagaraPosition	SpritePosition		= PositionReader.GetSafe(iParticle, DefaultPosition);
		const FLinearColor		SpriteColor			= ColorReader.GetSafe(iParticle, DefaultColor);
		const FVector2f			SpriteSize			= RenderContext.SizeToScreen(SpriteSizeReader.GetSafe(iParticle, DefaultSpriteSize));
		const float				SpriteRotation		= SpriteRotationReader.GetSafe(iParticle, DefaultSpriteRotation);
		const FVector4f			SpriteDynamicParam	= DynamicParamReader.GetSafe(iParticle, DefaultDynamicParam);

		const FVector2f SpriteCorners[] =
		{
			FVector2f(-SpriteSize.X, -SpriteSize.Y),
			FVector2f( SpriteSize.X, -SpriteSize.Y),
			FVector2f( SpriteSize.X,  SpriteSize.Y),
			FVector2f(-SpriteSize.X,  SpriteSize.Y),
		};

		FVector4f SpritePackedUV(0.0f, 0.0f, 1.0f, 1.0f);
		if (SubUVNumFrames > 1)
		{
			const float SpriteSubImageIndex = SubImageIndexReader.GetSafe(iParticle, DefaultSubImageIndex);

			const int32 FrameA = FMath::FloorToInt32(SpriteSubImageIndex) % SubUVNumFrames;
			const int32 ColA = FrameA % SubUVNumCols;
			const int32 RowA = FrameA / SubUVNumCols;

			SpritePackedUV.X = ColA * SubUVInvCols;
			SpritePackedUV.Y = RowA * SubUVInvRows;
			SpritePackedUV.Z = SpritePackedUV.X + SubUVInvCols;
			SpritePackedUV.W = SpritePackedUV.Y + SubUVInvRows;
		}

		const FVector2f SpriteUVs[] =
		{
			FVector2f(SpritePackedUV.X, SpritePackedUV.Y),
			FVector2f(SpritePackedUV.Z, SpritePackedUV.Y),
			FVector2f(SpritePackedUV.Z, SpritePackedUV.W),
			FVector2f(SpritePackedUV.X, SpritePackedUV.W),
		};

		const float CosRot = FMath::Cos(SpriteRotation);
		const float SinRot = FMath::Sin(SpriteRotation);

		const FVector2f Position = RenderContext.PositionToScreen(SpritePosition);

		const int32 VertexOffset = iInstance * 4;
		for ( int32 iVertex=0; iVertex < 4; ++iVertex)
		{
			FSlateVertex& Vertex		= ScratchBuffers.ScratchVertices[VertexOffset + iVertex];
			Vertex.Position.X			= Position.X + ( SpriteCorners[iVertex].X * CosRot) + (SpriteCorners[iVertex].Y * SinRot);
			Vertex.Position.Y			= Position.Y + (-SpriteCorners[iVertex].X * SinRot) + (SpriteCorners[iVertex].Y * CosRot);
			Vertex.TexCoords[0]			= SpriteUVs[iVertex].X;
			Vertex.TexCoords[1]			= SpriteUVs[iVertex].Y;
			Vertex.TexCoords[2]			= SpriteDynamicParam.X;
			Vertex.TexCoords[3]			= SpriteDynamicParam.Y;
			Vertex.MaterialTexCoords.X	= SpriteDynamicParam.Z;
			Vertex.MaterialTexCoords.Y	= SpriteDynamicParam.W;
			Vertex.PixelSize[0]			= 0;
			Vertex.PixelSize[1]			= 0;
			Vertex.Color				= SpriteColor.ToFColor(false);		// Not sRGB
		}

		const int32 IndexOffset = iInstance * 6;
		ScratchBuffers.ScratchIndices[IndexOffset + 0] = VertexOffset + 0;
		ScratchBuffers.ScratchIndices[IndexOffset + 1] = VertexOffset + 1;
		ScratchBuffers.ScratchIndices[IndexOffset + 2] = VertexOffset + 2;

		ScratchBuffers.ScratchIndices[IndexOffset + 3] = VertexOffset + 0;
		ScratchBuffers.ScratchIndices[IndexOffset + 4] = VertexOffset + 2;
		ScratchBuffers.ScratchIndices[IndexOffset + 5] = VertexOffset + 3;
	}

	RenderContext.DrawCustomVerts(Material);
}

#undef LOCTEXT_NAMESPACE
