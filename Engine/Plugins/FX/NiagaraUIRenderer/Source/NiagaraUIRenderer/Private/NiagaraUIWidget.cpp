// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraUIWidget.h"
#include "SNiagaraUIWidget.h"
#include "NiagaraUIComponent.h"

#include "NiagaraSystem.h"

#include "Engine/World.h"

UNiagaraUIWidget::UNiagaraUIWidget(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

#if WITH_EDITOR
void UNiagaraUIWidget::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.MemberProperty)
	{
		const FName PropertyName = PropertyChangedEvent.MemberProperty->GetFName();
		//NiagaraSystem

		//-todo:
		//RecreateOrInitialize();
		//RebuildWidget();
		//ReleaseSlateResources(true);
		//TakeWidget();
	}
}
#endif //WITH_EDITOR

TSharedRef<SWidget> UNiagaraUIWidget::RebuildWidget()
{
	NiagaraWidget = SNew(SNiagaraUIWidget);

	RecreateOrInitialize();

	return NiagaraWidget.ToSharedRef();
}

void UNiagaraUIWidget::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if (!NiagaraWidget.IsValid())
	{
		return;
	}

	RecreateOrInitialize();
}

void UNiagaraUIWidget::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	NiagaraWidget.Reset();
	if (NiagaraComponent)
	{
		NiagaraComponent->DestroyComponent();
		NiagaraComponent = nullptr;
	}
}

#if WITH_EDITOR
const FText UNiagaraUIWidget::GetPaletteCategory()
{
	return FText::FromString(TEXT("Niagara"));
}
#endif

void UNiagaraUIWidget::SetDesiredWidgetSize(FVector2D InSize)
{
	DesiredWidgetSize = InSize;
	if (NiagaraWidget.IsValid())
	{
		NiagaraWidget->SetDesiredSize(InSize);
	}
}

void UNiagaraUIWidget::RecreateOrInitialize()
{
	UWorld* World = GetWorld();
	if (!World || !World->PersistentLevel || World->bIsTearingDown || !NiagaraSystem)
	{
		if (NiagaraComponent != nullptr)
		{
			NiagaraComponent->DestroyComponent();
			NiagaraComponent = nullptr;
		}

		return;
	}

	if (NiagaraComponent == nullptr)
	{
		NiagaraComponent = NewObject<UNiagaraUIComponent>(this);
	}
	NiagaraComponent->SetAsset(NiagaraSystem);

	if (NiagaraWidget)
	{
		if (!NiagaraComponent->IsRegistered())
		{
			NiagaraComponent->RegisterComponentWithWorld(World);
		}
		NiagaraWidget->SetNiagaraComponent(NiagaraComponent);
		NiagaraWidget->SetDesiredSize(DesiredWidgetSize);
		NiagaraWidget->SetWorldToScreenScale(WorldToScreenScale);
		NiagaraWidget->SetWorldToScreenPlane(WorldToScreenPlane);
		NiagaraWidget->SetHorizontalAlignment(HorizontalAlignment);
		NiagaraWidget->SetVerticalAlignment(VerticalAlignment);
	}
}
