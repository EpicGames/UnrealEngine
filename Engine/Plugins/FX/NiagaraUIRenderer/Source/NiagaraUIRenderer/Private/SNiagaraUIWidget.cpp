// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraUIWidget.h"

#include "NiagaraUIComponent.h"
#include "NiagaraUITypes.h"
#include "NiagaraUIRendererProperties.h"
#include "NiagaraUISlateRenderContext.h"

#include "Engine/World.h"
#include "Framework/Application/SlateApplication.h"
#include "Rendering/DrawElementTypes.h"

void SNiagaraUIWidget::Construct(const FArguments& InArgs)
{
#if WITH_EDITOR
	MaterialCompilationFinishedHandle = UMaterial::OnMaterialCompilationFinished().AddLambda(
		[this](UMaterialInterface* ModifiedMaterial)
		{
			MaterialHandleCache.Empty();
		}
	);
#endif
}

SNiagaraUIWidget::~SNiagaraUIWidget()
{
#if WITH_EDITOR
	UMaterial::OnMaterialCompilationFinished().Remove(MaterialCompilationFinishedHandle);
#endif
}

void SNiagaraUIWidget::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	Super::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	UNiagaraUIComponent* NiagaraComponent = WeakNiagaraComponent.Get();
	if (NiagaraComponent)
	{
		NiagaraComponent->TickComponent(InDeltaTime, ELevelTick::LEVELTICK_All, nullptr);
	}
}

int32 SNiagaraUIWidget::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	UNiagaraUIComponent* NiagaraComponent = WeakNiagaraComponent.Get();
	const FNiagaraUIRenderData* RenderData = NiagaraComponent ? NiagaraComponent->GetRenderData() : nullptr;
	if (!RenderData)
	{
		return LayerId;
	}

	FNiagaraUISlateRenderContext RenderContext(MaterialHandleCache, AllottedGeometry, OutDrawElements);
	RenderContext.SetLayerId(LayerId);
	RenderContext.SetDrawEffect(bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect);
	RenderContext.SetScreenParameters(WorldToScreenPlane);
	RenderContext.SetScreenOriginAlignment(HorizontalAlignment, VerticalAlignment);
	RenderContext.SetScreenScale(WorldToScreenScale);

	for (const TUniquePtr<FNiagaraUIRendererRenderData>& RendererRenderData : RenderData->RendererRenderDatas)
	{
		const UNiagaraUIRendererProperties* Properties = RendererRenderData->WeakProperties.Get();
		if (Properties == nullptr)
		{
			continue;
		}

		Properties->ExecuteRender(RenderContext, *RendererRenderData.Get());
	}

	return LayerId;
}

FVector2D SNiagaraUIWidget::ComputeDesiredSize(float /*LayoutScaleMultiplier*/) const
{
	return DesiredWidgetSize;
}

void SNiagaraUIWidget::SetNiagaraComponent(UNiagaraUIComponent* InComponent)
{
	WeakNiagaraComponent = InComponent;
	if (InComponent)
	{
		InComponent->Activate();
	}
}

void SNiagaraUIWidget::SetDesiredSize(FVector2D InSize)
{
	DesiredWidgetSize = InSize;
}

void SNiagaraUIWidget::SetWorldToScreenScale(float InScale)
{
	WorldToScreenScale = InScale;
}

void SNiagaraUIWidget::SetWorldToScreenPlane(ENiagaraUIScreenPlane InScreenPlane)
{
	WorldToScreenPlane = InScreenPlane;
}

void SNiagaraUIWidget::SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment)
{
	HorizontalAlignment = InHorizontalAlignment;
}

void SNiagaraUIWidget::SetVerticalAlignment(EVerticalAlignment InVerticalAlignment)
{
	VerticalAlignment = InVerticalAlignment;
}

