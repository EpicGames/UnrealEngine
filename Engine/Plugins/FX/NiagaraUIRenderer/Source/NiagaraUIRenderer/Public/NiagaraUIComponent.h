// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "NiagaraComponent.h"
#include "NiagaraUITypes.h"
#include "NiagaraUIComponent.generated.h"

UCLASS(ClassGroup=FX, meta=(BlueprintSpawnableComponent, DisplayName="Niagara UI Component"))
class UNiagaraUIComponent : public UNiagaraComponent
{
	GENERATED_BODY()

public:
	UNiagaraUIComponent();

	// BEGIN: UNiagaraComponent interface
	virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	virtual void TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	// END: UNiagaraComponent interface

	// Get the last generated UI render data
	const FNiagaraUIRenderData* GetRenderData() const;

private:
	void CreateRenderData();

private:
	TUniquePtr<FNiagaraUIRenderData> RenderData;
};
