// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraUIDebugRenderer.h"

#include "NiagaraUIRendererProperties.h"
#include "NiagaraUIRenderContext.h"

#include "Async/Async.h"
#include "Components/LineBatchComponent.h"
#include "Engine/World.h"

#if WITH_NIAGARA_RENDERER_DEBUGDRAW 

namespace NiagaraUIDebugRendererPrivate
{
	static const FGeometry DebugGeometry(FVector2f(0.0f, 0.0f), FVector2f(-64.0f, -64.0f), FVector2f(128.0f, 128.0f), 1.0f);
}

class FNiagaraUIDebugRenderContext : public FNiagaraUIRenderContext
{
public:
	FNiagaraUIDebugRenderContext(FMaterialCacheMap& TempCacheMap, ULineBatchComponent& InLineBatcher)
		: FNiagaraUIRenderContext(TempCacheMap, NiagaraUIDebugRendererPrivate::DebugGeometry)
		, LineBatcher(InLineBatcher)
	{
	}

	virtual void DrawCustomVerts(const UMaterialInterface* Material, const TArray<FSlateVertex>& Vertices, const TArray<SlateIndex>& Indices) const override
	{
		TArray<FBatchedLine> Lines;

		const int32 NumTriangles = Indices.Num() / 3;
		for (int iTriangle = 0; iTriangle < NumTriangles; ++iTriangle)
		{
			const FSlateVertex& Vertex0 = Vertices[Indices[iTriangle * 3 + 0]];
			const FSlateVertex& Vertex1 = Vertices[Indices[iTriangle * 3 + 1]];
			const FSlateVertex& Vertex2 = Vertices[Indices[iTriangle * 3 + 2]];

			const FVector Positions[] =
			{
				FVector(Vertex0.Position.X, 0.0f, -Vertex0.Position.Y),
				FVector(Vertex1.Position.X, 0.0f, -Vertex1.Position.Y),
				FVector(Vertex2.Position.X, 0.0f, -Vertex2.Position.Y),
			};

			Lines.Emplace(Positions[0], Positions[1], FLinearColor::White, 0.0f, 1.0f, 0);
			Lines.Emplace(Positions[1], Positions[2], FLinearColor::White, 0.0f, 1.0f, 0);
			Lines.Emplace(Positions[2], Positions[0], FLinearColor::White, 0.0f, 1.0f, 0);
		}

		AsyncTask(
			ENamedThreads::GameThread,
			[WeakLineBatcher=MakeWeakObjectPtr(&LineBatcher), TaskLines=MoveTemp(Lines)]() mutable
			{
				if ( ULineBatchComponent* LineBatcher = WeakLineBatcher.Get() )
				{
					LineBatcher->DrawLines(TaskLines);
				}
			}
		);
	}

private:
	ULineBatchComponent&	LineBatcher;
};


FNiagaraUIDebugRenderer::FNiagaraUIDebugRenderer(ERHIFeatureLevel::Type FeatureLevel, const UNiagaraRendererProperties* InProps, const FNiagaraEmitterInstance* Emitter)
	: FNiagaraRenderer(FeatureLevel, InProps, Emitter)
{
}

FNiagaraDynamicDataBase* FNiagaraUIDebugRenderer::GenerateDynamicData(const FNiagaraSceneProxy* Proxy, const UNiagaraRendererProperties* InProperties, const FNiagaraEmitterInstance* Emitter) const
{
	const UNiagaraUIRendererProperties* RendererProperties = CastChecked<const UNiagaraUIRendererProperties>(InProperties);
	if (!RendererProperties->IsDebugDrawEnabled())
	{
		return nullptr;
	}

	FNiagaraSystemInstance* SystemInstance = Emitter->GetParentSystemInstance();
	UWorld* World = SystemInstance ? SystemInstance->GetWorld() : nullptr;
	ULineBatchComponent* LineBatcher = World ? World->GetLineBatcher(UWorld::ELineBatcherType::World) : nullptr;
	if (!LineBatcher)
	{
		return nullptr;
	}

	TUniquePtr<FNiagaraUIRendererRenderData> RenderData(RendererProperties->CreateRenderData(*Emitter));
	if (!RenderData.IsValid())
	{
		return nullptr;
	}

	FNiagaraUIRenderContext::FMaterialCacheMap TempCacheMap;
	FNiagaraUIDebugRenderContext RenderContext(TempCacheMap , *LineBatcher);
	RendererProperties->ExecuteRender(RenderContext, *RenderData.Get());

	return nullptr;
}
#endif
