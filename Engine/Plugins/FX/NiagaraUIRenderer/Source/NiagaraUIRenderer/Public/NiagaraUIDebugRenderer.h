// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraRendererProperties.h"
#include "NiagaraCommon.h"
#include "NiagaraGPUSortInfo.h"
#include "NiagaraSystemInstanceController.h"
#include "NiagaraUITypes.h"

#if WITH_NIAGARA_RENDERER_DEBUGDRAW 
class FNiagaraUIDebugRenderer : public FNiagaraRenderer
{
public:
	explicit FNiagaraUIDebugRenderer(ERHIFeatureLevel::Type FeatureLevel, const UNiagaraRendererProperties* InProps, const FNiagaraEmitterInstance* Emitter);
	virtual ~FNiagaraUIDebugRenderer() = default;
	
	//FNiagaraRenderer interface
	virtual FNiagaraDynamicDataBase* GenerateDynamicData(const FNiagaraSceneProxy* Proxy, const UNiagaraRendererProperties* InProperties, const FNiagaraEmitterInstance* Emitter) const override;
	//FNiagaraRenderer interface END
};
#endif
