// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraCommon.h"
#include "NiagaraDataSetAccessor.h"
#include "NiagaraGPUSortInfo.h"

#include "NiagaraUITypes.h"

#include "Rendering/RenderingCommon.h"
#include "Rendering/SlateResourceHandle.h"

class UMaterialInterface;

struct FNiagaraUIRendererScratchBuffers
{
	TArray<FSlateVertex>&	ScratchVertices;
	TArray<SlateIndex>&		ScratchIndices;
};

class FNiagaraUIRenderContext
{
public:
	using FMaterialCacheMap = TMap<FObjectKey, FSlateResourceHandle>;

public:
	explicit FNiagaraUIRenderContext(FMaterialCacheMap& InCacheMap, const FGeometry& InAllottedGeometry);
	virtual ~FNiagaraUIRenderContext() = default;

	// Get scratch buffers for generating vertex data into
	FNiagaraUIRendererScratchBuffers GetScratchBuffers(int32 NumVertices, int32 NumInidices) const;

	// Genrate an indirection table for particles both sorting and culling particles
	TConstArrayView<int32> CreateParticleOrderTable(int32 NumInstances, ENiagaraSortMode SortMode, FNiagaraDataSetReaderFloat<FNiagaraPosition> PositionReader, FNiagaraDataSetReaderFloat<float> SortOrderReader, int32 RendererVisibility, FNiagaraDataSetReaderInt32<int32> VisTagReader) const;

	// Draw the vertices build in the scratch buffer, will clear the buffer post draw
	void DrawCustomVerts(const UMaterialInterface* Material) const;

	// Draw the verticies provided
	virtual void DrawCustomVerts(const UMaterialInterface* Material, const TArray<FSlateVertex>& Vertices, const TArray<SlateIndex>& Indices) const = 0;

	// Convert Simulation Posiiton to Widget Screen
	FVector2f PositionToScreen(const FNiagaraPosition& WorldPosition) const
	{
		FVector2f ScreenPosition;
		ScreenPosition.X = (ScreenPlane == ENiagaraUIScreenPlane::YZ ? WorldPosition.Y : WorldPosition.X) *  ScreenScale + ScreenOrigin.X;
		ScreenPosition.Y = (ScreenPlane == ENiagaraUIScreenPlane::XY ? WorldPosition.Y : WorldPosition.Z) * -ScreenScale + ScreenOrigin.Y;
		return FVector2f(AllottedGeometry.LocalToAbsolute(FVector2D(ScreenPosition)));
	}

	FVector2f SizeToScreen(const FVector2f& WorldSize) const
	{
		return WorldSize * ScreenScale;
	}

	// Set functions to override default behavior
	void SetLayerId(int32 InLayerId) { LayerId = InLayerId; }
	void SetDrawEffect(ESlateDrawEffect InDrawEffect) { DrawEffect = InDrawEffect; }
	void SetScreenParameters(ENiagaraUIScreenPlane InScreenPlane) { ScreenPlane = InScreenPlane; }
	void SetScreenOrigin(const FVector2f& InScreenOrigin) { ScreenOrigin = InScreenOrigin; }
	void SetScreenOriginAlignment(EHorizontalAlignment HorizontalAlignment, EVerticalAlignment VerticalAlignment);
	void SetScreenScale(float InScreenScale) { ScreenScale = InScreenScale; }

protected:
	const FSlateResourceHandle& GetOrCreateMaterialHandle(const UMaterialInterface* Material) const;

protected:
	mutable TArray<FSlateVertex>	ScratchVertices;
	mutable TArray<SlateIndex>		ScratchIndices;
	mutable TArray<int32>			ParticleOrderTable;

	FMaterialCacheMap&				MaterialHandleCache;
	const FGeometry&				AllottedGeometry;
	int32							LayerId = 0;
	ESlateDrawEffect				DrawEffect = ESlateDrawEffect::None;

	ENiagaraUIScreenPlane			ScreenPlane = ENiagaraUIScreenPlane::XZ;
	FVector2f						ScreenOrigin = FVector2f::Zero();
	float							ScreenScale = 1.0f;
};
