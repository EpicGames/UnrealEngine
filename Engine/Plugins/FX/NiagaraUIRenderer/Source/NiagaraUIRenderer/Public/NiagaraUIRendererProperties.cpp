// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraUIRendererProperties.h"
#include "NiagaraUIDebugRenderer.h"

FNiagaraRenderer* UNiagaraUIRendererProperties::CreateEmitterRenderer(ERHIFeatureLevel::Type FeatureLevel, const FNiagaraEmitterInstance* Emitter, const FNiagaraSystemInstanceController& InController)
{
#if WITH_NIAGARA_RENDERER_DEBUGDRAW 
	return new FNiagaraUIDebugRenderer(FeatureLevel, this, Emitter);
#else
	return nullptr;
#endif
}

FNiagaraBoundsCalculator* UNiagaraUIRendererProperties::CreateBoundsCalculator()
{
	return nullptr;
}

bool UNiagaraUIRendererProperties::IsSimTargetSupported(ENiagaraSimTarget InSimTarget) const
{
	return InSimTarget == ENiagaraSimTarget::CPUSim;
}
