// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraRendererProperties.h"
#include "NiagaraCommon.h"
#include "NiagaraGPUSortInfo.h"
#include "NiagaraSystemInstanceController.h"
#include "NiagaraUITypes.h"
#include "NiagaraUIRendererProperties.generated.h"

class FNiagaraUIRenderContext;

UCLASS(ABSTRACT, MinimalAPI)
class UNiagaraUIRendererProperties : public UNiagaraRendererProperties
{
	GENERATED_BODY()

public:
	//~ BEGIN: UNiagaraRendererProperties interface
	virtual FNiagaraRenderer* CreateEmitterRenderer(ERHIFeatureLevel::Type FeatureLevel, const FNiagaraEmitterInstance* Emitter, const FNiagaraSystemInstanceController& InController) final override;
	virtual FNiagaraBoundsCalculator* CreateBoundsCalculator() final override;
	virtual bool IsSimTargetSupported(ENiagaraSimTarget InSimTarget) const final override;
#if WITH_NIAGARA_RENDERER_DEBUGDRAW 
	virtual bool SupportsDebugDraw() const { return true; }
#endif
	//~ END: UNiagaraRendererProperties interface

	virtual FNiagaraUIRendererRenderData* CreateRenderData(const FNiagaraEmitterInstance&) const { return nullptr; }
	virtual void ExecuteRender(const FNiagaraUIRenderContext& RenderContext, const FNiagaraUIRendererRenderData& RendererRenderData) const {}
};
