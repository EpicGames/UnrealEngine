// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraUIRendererProperties.h"
#include "NiagaraUISpriteRendererProperties.generated.h"

// Niagara Renderer for sprites in the UI (Slate / UMG)
UCLASS(EditInlineNew, meta=(DisplayName="UI Sprite Renderer", PrioritizeCategories="Material Bindings SubUV Sorting"))
class UNiagaraUISpriteRendererProperties : public UNiagaraUIRendererProperties
{
	GENERATED_BODY()

public:
	UNiagaraUISpriteRendererProperties();

	//~ BEGIN: UNiagaraRendererProperties interface
	virtual void GetUsedMaterials(const FNiagaraEmitterInstance* EmitterInstance, TArray<UMaterialInterface*>& OutMaterials) const override;
	virtual void PostInitProperties() override;
	virtual void PostLoad() override;
#if WITH_EDITORONLY_DATA
	virtual void GetRendererWidgets(const FNiagaraEmitterInstance* InEmitter, TArray<TSharedPtr<SWidget>>& OutWidgets, TSharedPtr<FAssetThumbnailPool> InThumbnailPool) const override;
	virtual void GetRendererTooltipWidgets(const FNiagaraEmitterInstance* InEmitter, TArray<TSharedPtr<SWidget>>& OutWidgets, TSharedPtr<FAssetThumbnailPool> InThumbnailPool) const override;
	virtual void GetRendererFeedback(const FVersionedNiagaraEmitter& InEmitter, TArray<FNiagaraRendererFeedback>& OutErrors, TArray<FNiagaraRendererFeedback>& OutWarnings, TArray<FNiagaraRendererFeedback>& OutInfo) const override;
#endif
	virtual void CacheFromCompiledData(const FNiagaraDataSetCompiledData* CompiledData) override;
	//~ END: UNiagaraRendererProperties interface

	//~ BEGIN: UNiagaraUIRendererProperties interface
	virtual FNiagaraUIRendererRenderData* CreateRenderData(const FNiagaraEmitterInstance& EmitterInstance) const override;
	virtual void ExecuteRender(const FNiagaraUIRenderContext& RenderContext, const FNiagaraUIRendererRenderData& RendererRenderData) const override;
	//~ END: UNiagaraUIRendererProperties interface

	void InitBindings();

	static void ProcessDeferredInit();

public:
	UPROPERTY(EditAnywhere, Category="Material")//, meta=(RequiredAssetDataTags="MaterialDomain=UI"))
	TObjectPtr<UMaterialInterface> Material;

	UPROPERTY(EditAnywhere, Category="SubUV", meta=(ClampMin="1"))
	FVector2f SubImageSize = FVector2f(1.0f, 1.0f);

	//UPROPERTY(EditAnywhere, Category="SubUV", meta=(DisplayName="Sub UV Blending Enabled"))
	//bool bSubImageBlend = false;

	UPROPERTY(EditAnywhere, Category="Sorting And Visibility")
	ENiagaraSortMode SortMode = ENiagaraSortMode::None;

	/** If a render visibility tag is present, particles whose tag matches this value will be visible in this renderer. */
	UPROPERTY(EditAnywhere, Category = "Sorting And Visibility")
	uint32 RendererVisibility = 0;

	UPROPERTY(EditAnywhere, Category="Bindings")
	FNiagaraVariableAttributeBinding PositionBinding;

	UPROPERTY(EditAnywhere, Category="Bindings")
	FNiagaraVariableAttributeBinding ColorBinding;

	UPROPERTY(EditAnywhere, Category="Bindings")
	FNiagaraVariableAttributeBinding SpriteSizeBinding;

	UPROPERTY(EditAnywhere, Category="Bindings")
	FNiagaraVariableAttributeBinding SpriteRotationBinding;

	UPROPERTY(EditAnywhere, Category="Bindings")
	FNiagaraVariableAttributeBinding SubImageIndexBinding;

	// Packed into TexCoord[1] / TexCoord[2]
	UPROPERTY(EditAnywhere, Category="Bindings")
	FNiagaraVariableAttributeBinding DynamicMaterialParameterBinding;

	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding CustomSortingBinding;

	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding RendererVisibilityTagBinding;

private:
	FNiagaraDataSetAccessor<FNiagaraPosition>	PositionDataSetAccessor;
	FNiagaraDataSetAccessor<FLinearColor>		ColorDataSetAccessor;
	FNiagaraDataSetAccessor<FVector2f>			SpriteSizeDataSetAccessor;
	FNiagaraDataSetAccessor<float>				SpriteRotationDataSetAccessor;
	FNiagaraDataSetAccessor<float>				SubImageIndexDataSetAccessor;
	FNiagaraDataSetAccessor<FVector4f>			DynamicMaterialParameterDataSetAccessor;
	FNiagaraDataSetAccessor<float>				CustomSortingDataSetAccessor;
	FNiagaraDataSetAccessor<int32>				RendererVisibilityTagDataSetAccessor;
};
