// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraCommon.h"
#include "NiagaraUITypes.generated.h"

class UMaterialInterface;
class UNiagaraUIRendererProperties;

/** Which world-space plane to project Niagara particle positions onto. */
UENUM(BlueprintType)
enum class ENiagaraUIScreenPlane : uint8
{
	/** Project X (right) and Y (up) — typical for effects simulated in the XY plane. */
	XY,
	/** Project X (right) and Z (up). */
	XZ,
	/** Project Y (right) and Z (up). */
	YZ,
};

struct FNiagaraUIRendererRenderData
{
	TWeakObjectPtr<const UNiagaraUIRendererProperties>	WeakProperties;
	TWeakObjectPtr<UMaterialInterface>					WeakMaterial;
	FNiagaraDataBufferRef								DataBuffer;
	int32												SortOrder = 0;
};

struct FNiagaraUIRenderData
{
	TArray<TUniquePtr<FNiagaraUIRendererRenderData>> RendererRenderDatas;
};

