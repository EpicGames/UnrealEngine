// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Components/Widget.h"
#include "NiagaraUITypes.h"
#include "NiagaraUIWidget.generated.h"

class UNiagaraUIComponent;
class SNiagaraUIWidget;
class UNiagaraSystem;

UCLASS(meta=(DisplayName="Niagara UI Widget"))
class UNiagaraUIWidget : public UWidget
{
	GENERATED_BODY()

public:
	UNiagaraUIWidget(const FObjectInitializer& ObjectInitializer);

	// BEGIN: UObject interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif //WITH_EDITOR
	// END: UObject interface

	// BEGIN: UWidget interface
	virtual TSharedRef<SWidget> RebuildWidget() override;
	virtual void SynchronizeProperties() override;
	virtual void ReleaseSlateResources(bool bReleaseChildren) override;
#if WITH_EDITOR
	virtual const FText GetPaletteCategory() override;
#endif
	// END: UWidget interface

	UFUNCTION(BlueprintCallable, Category = "Niagara")
	void SetDesiredWidgetSize(FVector2D InSize);

private:
	void RecreateOrInitialize();

protected:
	// Layout for widget, does not impact simulation coordinate translation
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "UI Rendering")
	FVector2D DesiredWidgetSize = FVector2D(100.0, 100.0);

	// Scale to translate Niagara units to UI units
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "UI Rendering")
	float WorldToScreenScale = 1.0f;

	// Axis to translate Niagara to UI
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "UI Rendering")
	ENiagaraUIScreenPlane WorldToScreenPlane = ENiagaraUIScreenPlane::XY;

	// Horizontal alignment that represents the world origin
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "UI Rendering")
	TEnumAsByte<EHorizontalAlignment> HorizontalAlignment = HAlign_Center;

	// Vertical alignment that represents the world origin
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "UI Rendering")
	TEnumAsByte<EVerticalAlignment> VerticalAlignment = VAlign_Center;

	// System simulation to use
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Niagara")
	TObjectPtr<UNiagaraSystem> NiagaraSystem;

	// Active component, used for rendering
	UPROPERTY(Transient)
	TObjectPtr<UNiagaraUIComponent> NiagaraComponent;

	TSharedPtr<SNiagaraUIWidget> NiagaraWidget;
};
