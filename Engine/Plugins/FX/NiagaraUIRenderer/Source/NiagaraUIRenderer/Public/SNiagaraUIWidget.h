// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Widgets/SLeafWidget.h"
#include "Rendering/SlateResourceHandle.h"
#include "NiagaraUITypes.h"

class UNiagaraUIComponent;
class UMaterialInterface;

class SNiagaraUIWidget : public SLeafWidget
{
	using Super = SLeafWidget;

public:
	SLATE_BEGIN_ARGS(SNiagaraUIWidget) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	virtual ~SNiagaraUIWidget();

	// BEGIN: SWidget interface
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override;
	virtual bool ComputeVolatility() const override { return true; }
	// END: SWidget interface

	/** Switch which component this widget reads from. */
	void SetNiagaraComponent(UNiagaraUIComponent* InComponent);
	void SetDesiredSize(FVector2D InSize);
	void SetWorldToScreenScale(float InScale);
	void SetWorldToScreenPlane(ENiagaraUIScreenPlane InScreenPlane);
	void SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment);
	void SetVerticalAlignment(EVerticalAlignment InVerticalAlignment);

private:
	TWeakObjectPtr<UNiagaraUIComponent> WeakNiagaraComponent;

	FVector2D DesiredWidgetSize = FVector2D(100.0, 100.0);

	float					WorldToScreenScale = 1.0f;
	ENiagaraUIScreenPlane	WorldToScreenPlane = ENiagaraUIScreenPlane::XY;
	EHorizontalAlignment	HorizontalAlignment = HAlign_Center;
	EVerticalAlignment		VerticalAlignment = VAlign_Center;

	// Cached map from material to slate resource
	mutable TMap<FObjectKey, FSlateResourceHandle> MaterialHandleCache;

#if WITH_EDITOR
	FDelegateHandle MaterialCompilationFinishedHandle;
#endif
};
