// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class NiagaraUIRendererEditor : ModuleRules
{
	public NiagaraUIRendererEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"CoreUObject",
			"Engine",
			"NiagaraUIRenderer",
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"NiagaraEditor",
			"UnrealEd",
			"PropertyEditor",
			"SlateCore",
			"Slate",
		});
	}
}
