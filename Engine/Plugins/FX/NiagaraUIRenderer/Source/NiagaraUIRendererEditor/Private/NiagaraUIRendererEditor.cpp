// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraUISpriteRendererProperties.h"
#include "NiagaraEditorModule.h"
#include "Modules/ModuleManager.h"

class FNiagaraUIRendererEditorModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
		FNiagaraEditorModule& NiagaraEditor = FModuleManager::LoadModuleChecked<FNiagaraEditorModule>("NiagaraEditor");

		NiagaraEditor.RegisterRendererCreationInfo(FNiagaraRendererCreationInfo(
			UNiagaraUISpriteRendererProperties::StaticClass()->GetDisplayNameText(),
			FText::FromString(UNiagaraUISpriteRendererProperties::StaticClass()->GetDescription()),
			UNiagaraUISpriteRendererProperties::StaticClass()->GetClassPathName(),
			FNiagaraRendererCreationInfo::FRendererFactory::CreateLambda([](UObject* OuterEmitter)
			{
				return NewObject<UNiagaraUISpriteRendererProperties>(OuterEmitter, NAME_None, RF_Transactional);
			})));
	}

	virtual void ShutdownModule() override {}
};

IMPLEMENT_MODULE(FNiagaraUIRendererEditorModule, NiagaraUIRendererEditor)
